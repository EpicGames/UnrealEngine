// Copyright Epic Games, Inc. All Rights Reserved.

#include "FX/SlateFXBaseSubsystem.h"

#include "Engine/Engine.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SlateFXBaseSubsystem)

USlatePostBufferProcessor* USlateFXBaseSubsystem::GetPostProcessor(ESlatePostRT InSlatePostBufferBit)
{
	USlatePostBufferProcessor* Result = nullptr;

	if (GEngine)
	{
		if (USlateFXBaseSubsystem* SlateFXSubsystem = GEngine->GetEngineSubsystem<USlateFXBaseSubsystem>())
		{
			Result = SlateFXSubsystem->GetSlatePostProcessor(InSlatePostBufferBit);
		}
	}

	return Result;
}

bool USlateFXBaseSubsystem::ShouldCreateSubsystem(UObject* Outer) const
{
	TArray<UClass*> ChildClasses;
	GetDerivedClasses(GetClass(), ChildClasses, false);

	// Only create an instance if there is no override implementation defined elsewhere
	return ChildClasses.Num() == 0;
}

void USlateFXBaseSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);

	FWorldDelegates::OnPreWorldInitialization.AddUObject(this, &USlateFXBaseSubsystem::OnPreWorldInitialization);
	FWorldDelegates::OnPostWorldCleanup.AddUObject(this, &USlateFXBaseSubsystem::OnPostWorldCleanup);
}

void USlateFXBaseSubsystem::Deinitialize()
{
	SlatePostBufferProcessors.Empty();

	FWorldDelegates::OnPreWorldInitialization.RemoveAll(this);
	FWorldDelegates::OnPostWorldCleanup.RemoveAll(this);

	Super::Deinitialize();
}

USlatePostBufferProcessor* USlateFXBaseSubsystem::GetSlatePostProcessor(ESlatePostRT InPostBufferBit)
{
	if (TObjectPtr<USlatePostBufferProcessor>* Processor = SlatePostBufferProcessors.Find(InPostBufferBit))
	{
		return *Processor;
	}

	return nullptr;
}

void USlateFXBaseSubsystem::OnPreWorldInitialization(UWorld* World, const UWorld::InitializationValues IVS)
{
	if (World && World->WorldType == EWorldType::EditorPreview)
	{
		return;
	}

	SlatePostBufferProcessors.Empty();
	OnInitProcessors();
}

void USlateFXBaseSubsystem::OnPostWorldCleanup(UWorld* World, bool SessionEnded, bool bCleanupResources)
{
	if (World && World->WorldType == EWorldType::EditorPreview)
	{
		return;
	}

	SlatePostBufferProcessors.Empty();
	OnCleanupProcessors();
}
