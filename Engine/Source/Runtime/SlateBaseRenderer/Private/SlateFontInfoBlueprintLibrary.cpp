// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlateFontInfoBlueprintLibrary.h"
#include "Blueprint/BlueprintExceptionInfo.h"
#include "Materials/MaterialInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SlateFontInfoBlueprintLibrary)

DEFINE_LOG_CATEGORY(LogSlateFontInfoBlueprint);


FSlateFontInfo USlateFontInfoBlueprintLibrary::MakeSlateFontInfo(const UObject* FontObject, UObject* FontMaterial, FFontOutlineSettings OutlineSettings, FName TypefaceFontName, float Size, int32 LetterSpacing, float SkewAmount, bool bForceMonospaced, bool bMaterialIsStencil, float MonospacedWidth)
{
	FSlateFontInfo SlateFontInfo = FSlateFontInfo();
	SlateFontInfo.FontObject = FontObject;
	SlateFontInfo.FontMaterial = FontMaterial;
	SlateFontInfo.OutlineSettings = OutlineSettings;
	SlateFontInfo.TypefaceFontName = TypefaceFontName;
	SlateFontInfo.Size = Size;
	SlateFontInfo.LetterSpacing = LetterSpacing;
	SlateFontInfo.SkewAmount = SkewAmount;
	SlateFontInfo.bForceMonospaced = bForceMonospaced;
	SlateFontInfo.bMaterialIsStencil = bMaterialIsStencil;
	SlateFontInfo.MonospacedWidth = MonospacedWidth;

	if (FontMaterial && !FontMaterial->IsA<UMaterialInterface>())
	{
#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST
		if (FFrame::GetThreadLocalTopStackFrame() && FFrame::GetThreadLocalTopStackFrame()->Object)
		{
			FText ErrorMessage = FText::FromString(TEXT("Font Material is not of type material. Either remove the reference or replace it with an object of type material."));
			const FBlueprintExceptionInfo ExceptionInfo(EBlueprintExceptionType::FatalError, ErrorMessage);
			FBlueprintCoreDelegates::ThrowScriptException(FFrame::GetThreadLocalTopStackFrame()->Object, *FFrame::GetThreadLocalTopStackFrame(), ExceptionInfo);
		}
		else
#endif
		{
			UE_LOG(LogSlateFontInfoBlueprint, Warning, TEXT("Font Material is not of type material. It will be replaced with no material."));
		}
		SlateFontInfo.FontMaterial = nullptr;
	}

	return SlateFontInfo;
}
