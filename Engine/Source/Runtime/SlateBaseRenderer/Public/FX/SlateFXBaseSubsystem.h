// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Subsystems/EngineSubsystem.h"
#include "Engine/World.h"
#include "Rendering/SlateRendererTypes.h"

#include "SlateFXBaseSubsystem.generated.h"

#define UE_API SLATEBASERENDERER_API

class USlatePostBufferProcessor;

UCLASS(MinimalAPI, DisplayName = "Slate FX Subsystem")
class USlateFXBaseSubsystem : public UEngineSubsystem
{
	GENERATED_BODY()

public:

	/** Get post processor for a particular post buffer index, if it exists */
	static UE_API USlatePostBufferProcessor* GetPostProcessor(ESlatePostRT InSlatePostBufferBit);

	//~Begin UEngineSubsystem Interface
	UE_API virtual bool ShouldCreateSubsystem(UObject* Outer) const override;
	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	UE_API virtual void Deinitialize() override;
	//~End UEngineSubsystem Interface

	/** Get post processor for a particular post buffer index, if it exists */
	UFUNCTION(BlueprintCallable, Category = "SlateFX")
	UE_API USlatePostBufferProcessor* GetSlatePostProcessor(ESlatePostRT InPostBufferBit);

protected:

	/** Called during world initialization to allow subclasses to populate SlatePostBufferProcessors */
	virtual void OnInitProcessors() {}

	/** Called during world cleanup to allow subclasses to release renderer-specific resources */
	virtual void OnCleanupProcessors() {}

	/** Map of post RT buffer index to buffer processors, if they exist */
	UPROPERTY(Transient)
	TMap<ESlatePostRT, TObjectPtr<USlatePostBufferProcessor>> SlatePostBufferProcessors;

private:

	/** Callback to create processors on world init */
	void OnPreWorldInitialization(UWorld* World, const UWorld::InitializationValues IVS);

	/** Callback to remove processors on world cleanup */
	void OnPostWorldCleanup(UWorld* World, bool SessionEnded, bool bCleanupResources);
};

#undef UE_API
