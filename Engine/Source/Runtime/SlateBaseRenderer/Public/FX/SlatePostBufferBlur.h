// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FX/SlatePostBufferProcessor.h"
#include "SlatePostBufferBlur.generated.h"

#define UE_API SLATEBASERENDERER_API

/**
 * Slate Post Buffer Processor that performs a simple gaussian blur to the backbuffer
 * 
 * Create a new asset deriving from this class to use / modify settings.
 */
UCLASS(MinimalAPI, Abstract, Blueprintable, CollapseCategories)
class USlatePostBufferBlur : public USlatePostBufferProcessor
{
	GENERATED_BODY()

public:

	UPROPERTY(interp, BlueprintReadWrite, Category = "GaussianBlur")
	float GaussianBlurStrength = 10;

public:

	USlatePostBufferBlur() {}
	virtual ~USlatePostBufferBlur() override {}
};

#undef UE_API
