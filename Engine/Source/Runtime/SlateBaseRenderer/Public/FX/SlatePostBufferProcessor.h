// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "SlatePostBufferProcessor.generated.h"

#define UE_API SLATEBASERENDERER_API

/**
 * Base class for proxy for post buffer processor that the renderthread uses to perform processing.
 */
class FSlatePostBufferProcessorProxy : public TSharedFromThis<FSlatePostBufferProcessorProxy>
{
public:
	virtual ~FSlatePostBufferProcessorProxy() {}
};

/**
 * Base class for types that can process the backbuffer scene into the slate post buffer.
 *
 * You need to register a proxy that derives from 'FSlateRHIPostBufferProcessorProxy' in USlateFXSubsystem::CreatePostBufferProcessorProxy
 * For an example see: USlatePostBufferBlur.
 */
UCLASS(MinimalAPI, Abstract, Blueprintable, CollapseCategories)
class USlatePostBufferProcessor : public UObject
{
	GENERATED_BODY()

public:

	virtual ~USlatePostBufferProcessor() = default;

	void SetRenderThreadProxy(TSharedPtr<FSlatePostBufferProcessorProxy> Proxy)
	{
		RenderThreadProxy = Proxy;
	}

private:
	TSharedPtr<FSlatePostBufferProcessorProxy> RenderThreadProxy;
};


#undef UE_API
