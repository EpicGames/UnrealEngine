// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Fonts/SlateFontInfo.h"
#include "UObject/Class.h"
#include "SlateFontInfoBlueprintLibrary.generated.h"

SLATEBASERENDERER_API DECLARE_LOG_CATEGORY_EXTERN(LogSlateFontInfoBlueprint, Log, All);

UCLASS()
class USlateFontInfoBlueprintLibrary : public UObject
{
	GENERATED_BODY()

	UFUNCTION(BlueprintPure, Category = "Slate Font", meta = (NativeMakeFunc))
	static SLATEBASERENDERER_API FSlateFontInfo MakeSlateFontInfo(const UObject* FontObject, UObject* FontMaterial, FFontOutlineSettings OutlineSettings, FName TypefaceFontName, float Size = 24.0f, int32 LetterSpacing = 0, float SkewAmount = 0.0f, bool bForceMonospaced = false, bool bMaterialIsStencil = false, float MonospacedWidth = 1.0f);
};
