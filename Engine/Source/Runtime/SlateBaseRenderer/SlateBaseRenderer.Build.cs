// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class SlateBaseRenderer : ModuleRules
{
	public SlateBaseRenderer(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Slate",
				"SlateCore"
			}
		);

		if (Target.bCompileAgainstEngine)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"Engine"
				});
		}
	}
}
